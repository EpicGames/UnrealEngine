// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncAvaSyncProvider.h"
#include "IAvaMediaModule.h"
#include "IStormSyncTransportClientModule.h"
#include "ModularFeature/AvaMediaSyncProviderFeatureTypes.h"
#include "Playback/IAvaPlaybackServer.h"
#include "Playback/IAvaPlaybackClient.h"
#include "StormSyncAvaBridgeCommon.h"
#include "StormSyncAvaBridgeLog.h"
#include "StormSyncCoreDelegates.h"

#define LOCTEXT_NAMESPACE "StormSyncAvaSyncProvider"

FStormSyncAvaSyncProvider::FStormSyncAvaSyncProvider()
{
	FStormSyncCoreDelegates::OnPakAssetExtracted.AddRaw(this, &FStormSyncAvaSyncProvider::OnPakAssetExtracted);
}

FStormSyncAvaSyncProvider::~FStormSyncAvaSyncProvider()
{
	FStormSyncCoreDelegates::OnPakAssetExtracted.RemoveAll(this);
}

FName FStormSyncAvaSyncProvider::GetName() const
{
	return TEXT("StormSyncAvaSyncProvider");
}

void FStormSyncAvaSyncProvider::SyncToAll(const TArray<FName>& InPackageNames)
{
	UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::SyncToAll InPackageNames: %d", InPackageNames.Num());

	const FStormSyncPackageDescriptor PackageDescriptor;
	IStormSyncTransportClientModule::Get().SynchronizePackages(PackageDescriptor, InPackageNames);
}

void FStormSyncAvaSyncProvider::PushToRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncResponse& DoneDelegate)
{
	UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::PushToRemote InRemoteName: %ls, InPackageNames: %d", *InRemoteName, InPackageNames.Num());
	
	FText ErrorText;
	FMessageAddress MessageAddress;
	// Get address from either client or server user data, depending on local client / server state
	if (!GetAddressFromUserData(InRemoteName, UE::StormSync::AvaBridgeCommon::StormSyncClientAddressKey, MessageAddress, &ErrorText))
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "FStormSyncAvaSyncProvider::PushToRemote - %ls", *ErrorText.ToString());
		DoneDelegate.ExecuteIfBound(CreateErrorResponse(ErrorText));
		return;
	}

	const FOnStormSyncPushComplete Delegate = FOnStormSyncPushComplete::CreateLambda([DoneDelegate](const TSharedPtr<FStormSyncTransportPushResponse>& Response)
	{
		if (!Response.IsValid())
		{
			DoneDelegate.ExecuteIfBound(CreateErrorResponse(LOCTEXT("Error_PushInvalidResponse", "Received invalid shared ptr response")));
			return;
		}
		
		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::PushToRemote - Response: %ls", *Response->ToString());

		if (DoneDelegate.IsBound())
		{
			FAvaMediaSyncResponse SyncResponse = ConvertSyncResponse(Response);
			DoneDelegate.Execute(MakeShared<FAvaMediaSyncResponse>(SyncResponse));
		}
	});

	const FStormSyncPackageDescriptor PackageDescriptor;
	IStormSyncTransportClientModule::Get().PushPackages(PackageDescriptor, InPackageNames, MessageAddress, Delegate);
}

void FStormSyncAvaSyncProvider::PullFromRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncResponse& DoneDelegate)
{
	UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::PullFromRemote InRemoteName: %ls, InPackageNames: %d", *InRemoteName, InPackageNames.Num());

	FText ErrorText;
	FMessageAddress MessageAddress;
	// Get address from either client or server user data, depending on local client / server state
	if (!GetAddressFromUserData(InRemoteName, UE::StormSync::AvaBridgeCommon::StormSyncClientAddressKey, MessageAddress, &ErrorText))
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "FStormSyncAvaSyncProvider::PullFromRemote - %ls", *ErrorText.ToString());
		DoneDelegate.ExecuteIfBound(CreateErrorResponse(ErrorText));
		return;
	}

	const FOnStormSyncPullComplete Delegate = FOnStormSyncPullComplete::CreateLambda([DoneDelegate](const TSharedPtr<FStormSyncTransportPullResponse>& Response)
	{
		if (!Response.IsValid())
		{
			DoneDelegate.ExecuteIfBound(CreateErrorResponse(LOCTEXT("Error_PullInvalidResponse", "Received invalid shared ptr response")));
			return;
		}
		
		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::PullFromRemote - Response: %ls", *Response->ToString());

		if (DoneDelegate.IsBound())
		{
			FAvaMediaSyncResponse SyncResponse = ConvertSyncResponse(Response);
			DoneDelegate.Execute(MakeShared<FAvaMediaSyncResponse>(SyncResponse));
		}
	});

	const FStormSyncPackageDescriptor PackageDescriptor;
	IStormSyncTransportClientModule::Get().PullPackages(PackageDescriptor, InPackageNames, MessageAddress, Delegate);
}

void FStormSyncAvaSyncProvider::CompareWithRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncCompareResponse& DoneDelegate)
{
	UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::CompareWithRemote InRemoteName: %ls, InPackageNames: %d", *InRemoteName, InPackageNames.Num());

	const bool bIsClient = IsPlaybackClient();
	const bool bIsServer = IsPlaybackServer();
	UE_LOGF(LogStormSyncAvaBridge, Display, "\t bIsClient: %ls, bIsServer: %ls", bIsClient ? TEXT("true") : TEXT("false"), bIsServer ? TEXT("true") : TEXT("false"));

	// Sanity check on input params
	if (InRemoteName.IsEmpty())
	{
		DoneDelegate.ExecuteIfBound(CreateErrorResponse(LOCTEXT("Error_TypeError_EmptyRemoteName", "Compare with remote called with an empty remote name")));
		return;
	}

	if (InPackageNames.IsEmpty())
	{
		DoneDelegate.ExecuteIfBound(CreateErrorResponse(LOCTEXT("Error_TypeError_EmptyPackageNames", "Compare with remote called with an empty list of package names")));
		return;
	}

	FText ErrorText;
	FMessageAddress MessageAddress;
	// Get address from either client or server user data, depending on local client / server state
	if (!GetAddressFromUserData(InRemoteName, UE::StormSync::AvaBridgeCommon::StormSyncClientAddressKey, MessageAddress, &ErrorText))
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "FStormSyncAvaSyncProvider::CompareWithRemote - %ls", *ErrorText.ToString());
		DoneDelegate.ExecuteIfBound(CreateErrorResponse(ErrorText));
		return;
	}
	
	// Issue request and build a response
	CompareWith(MessageAddress, InPackageNames, DoneDelegate);
}

FOnAvaMediaSyncPackageModified& FStormSyncAvaSyncProvider::GetOnAvaSyncPackageModified()
{
	return OnAvaSyncPackageModified;
}

void FStormSyncAvaSyncProvider::CompareWith(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncCompareResponse& DoneDelegate)
{
	UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::CompareWith InPackageNames: %d (Remote Address: %ls)", InPackageNames.Num(), *InRemoteAddress.ToString());

	for (const FName& PackageName : InPackageNames)
	{
		UE_LOGF(LogStormSyncAvaBridge, Display, "\t PackageNames: %ls", *PackageName.ToString());
	}

	IStormSyncTransportClientModule::Get().RequestPackagesStatus(InRemoteAddress, InPackageNames, FOnStormSyncRequestStatusComplete::CreateLambda([DoneDelegate](const TSharedPtr<FStormSyncTransportStatusResponse>& Response)
	{
		if (!Response.IsValid())
		{
			DoneDelegate.ExecuteIfBound(CreateErrorResponse(LOCTEXT("Error_CompareInvalidResponse", "Received invalid shared ptr response")));
			return;
		}

		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaSyncProvider::CompareWith - Response: %ls", *Response->ToString());

		if (DoneDelegate.IsBound())
		{
			FAvaMediaSyncCompareResponse CompareResponse;
			CompareResponse.Status = EAvaMediaSyncResponseResult::Success;
			CompareResponse.ConnectionInfo = ConvertConnectionInfo(Response->ConnectionInfo);
			CompareResponse.bNeedsSynchronization = Response->bNeedsSynchronization;
			DoneDelegate.Execute(MakeShared<FAvaMediaSyncCompareResponse>(CompareResponse));
		}
	}));
}

void FStormSyncAvaSyncProvider::OnPakAssetExtracted(const FName& InPackageName, const FString& InDestFilepath)
{
	OnAvaSyncPackageModified.Broadcast(InPackageName);
}

bool FStormSyncAvaSyncProvider::GetAddressFromUserData(const FString& InRemoteName, const FString& InUserDataKey, FMessageAddress& OutAddress, FText* OutErrorMessage)
{
	const bool bIsClient = IsPlaybackClient();
	const bool bIsServer = IsPlaybackServer();

	UE_LOGF(LogStormSyncAvaBridge, 
		Display,
		"FStormSyncAvaSyncProvider::GetAddressFromUserData InRemoteName: %ls, bIsClient: %ls, bIsServer: %ls",
		*InRemoteName,
		bIsClient ? TEXT("true") : TEXT("false"),
		bIsServer ? TEXT("true") : TEXT("false")
	);

	if (bIsClient)
	{
		// Also handle the case of both a client & server, in which case, the client takes priority and server side is ignored
		return GetAddressFromServerUserData(InRemoteName, InUserDataKey, OutAddress, OutErrorMessage);
	}

	if (bIsServer)
	{
		return GetAddressFromClientUserData(InRemoteName, InUserDataKey, OutAddress, OutErrorMessage);
	}

	// Case of neither a client or server
	if (OutErrorMessage)
	{
		*OutErrorMessage = LOCTEXT("Error_NoClientOrServer", "Local editor instance is not a valid client or server (not started)");
	}

	FMessageAddress Address;
	Address.Invalidate();
	OutAddress = Address;
	return false;
}

bool FStormSyncAvaSyncProvider::GetAddressFromServerUserData(const FString& InServerName, const FString& InUserDataKey, FMessageAddress& OutAddress, FText* OutErrorMessage)
{
	if (!IAvaMediaModule::Get().IsPlaybackClientStarted())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("Error_ReadingServerData_ClientNotStarted", "Trying to read server data from client but client is not started");
		}
		return false;
	}

	const IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	const FString AddressId = PlaybackClient.GetServerUserData(InServerName, InUserDataKey);
	if (AddressId.IsEmpty())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = FText::Format(
				LOCTEXT("Error_ReadingServerData_Empty", "Server user data Address Id is empty (ServerName: {0}, Key: {1})"),
				FText::FromString(InServerName),
				FText::FromString(InUserDataKey)
			);
		}
		return false;
	}

	return FMessageAddress::Parse(AddressId, OutAddress);
}

bool FStormSyncAvaSyncProvider::GetAddressFromClientUserData(const FString& InClientName, const FString& InUserDataKey, FMessageAddress& OutAddress, FText* OutErrorMessage)
{
	if (!IAvaMediaModule::Get().IsPlaybackServerStarted())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("Error_ReadingClientData_ServerNotStarted", "Trying to read client data from server but server is not started");
		}
		return false;
	}

	const IAvaPlaybackServer* PlaybackServer = IAvaMediaModule::Get().GetPlaybackServer();
	
	if (!PlaybackServer)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("Error_ReadingClientData_InvalidPlaybackServer", "Trying to read client data from server but server is not valid");
		}
		return false;
	}

	const FString AddressId = PlaybackServer->GetClientUserData(InClientName, InUserDataKey);
	if (AddressId.IsEmpty())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = FText::Format(
				LOCTEXT("Error_ReadingClientData_Empty", "Client user data Address Id is empty (ClientName: {0}, Key: {1})"),
				FText::FromString(InClientName),
				FText::FromString(InUserDataKey)
			);
		}
		return false;
	}

	return FMessageAddress::Parse(AddressId, OutAddress);
}

bool FStormSyncAvaSyncProvider::IsPlaybackServer()
{
	return IAvaMediaModule::Get().IsPlaybackServerStarted();
}

bool FStormSyncAvaSyncProvider::IsPlaybackClient()
{
	return IAvaMediaModule::Get().IsPlaybackClientStarted();
}

TSharedPtr<FAvaMediaSyncCompareResponse> FStormSyncAvaSyncProvider::CreateErrorResponse(const FText& InText)
{
	FAvaMediaSyncCompareResponse Response;
	Response.Status = EAvaMediaSyncResponseResult::Error;
	Response.ErrorText = InText;
	return MakeShared<FAvaMediaSyncCompareResponse>(Response);
}

FAvaMediaSyncResponse FStormSyncAvaSyncProvider::ConvertSyncResponse(const TSharedPtr<FStormSyncTransportSyncResponse>& InResponse)
{
	check(InResponse.IsValid());
	
	FAvaMediaSyncResponse AvaSyncResponse;
	if (InResponse->Status == EStormSyncResponseResult::Success)
	{
		AvaSyncResponse.Status = EAvaMediaSyncResponseResult::Success;
		AvaSyncResponse.StatusText = InResponse->StatusText;
	}
	else if (InResponse->Status == EStormSyncResponseResult::Error)
	{
		AvaSyncResponse.Status = EAvaMediaSyncResponseResult::Error;
		AvaSyncResponse.ErrorText = InResponse->StatusText;
	}

	AvaSyncResponse.ConnectionInfo = ConvertConnectionInfo(InResponse->ConnectionInfo);
	return AvaSyncResponse;
}

FAvaMediaSyncConnectionInfo FStormSyncAvaSyncProvider::ConvertConnectionInfo(const FStormSyncConnectionInfo& InConnectionInfo)
{
	FAvaMediaSyncConnectionInfo ConnectionInfo;
	ConnectionInfo.EngineVersion = InConnectionInfo.EngineVersion;
	ConnectionInfo.InstanceId = InConnectionInfo.InstanceId;
	ConnectionInfo.SessionId = InConnectionInfo.SessionId;
	ConnectionInfo.HostName = InConnectionInfo.HostName;
	ConnectionInfo.ProjectName = InConnectionInfo.ProjectName;
	ConnectionInfo.ProjectDir = InConnectionInfo.ProjectDir;
	
	// Deal with instance type different enum types
	ConnectionInfo.InstanceType = ConvertInstanceType(InConnectionInfo.InstanceType);
	return ConnectionInfo;
}

EAvaMediaSyncEngineType FStormSyncAvaSyncProvider::ConvertInstanceType(const EStormSyncEngineType InInstanceType)
{
	EAvaMediaSyncEngineType Result;
	switch (InInstanceType)
	{
	case EStormSyncEngineType::Server:
		Result = EAvaMediaSyncEngineType::Server;
		break;
	case EStormSyncEngineType::Commandlet:
		Result = EAvaMediaSyncEngineType::Commandlet;
		break;
	case EStormSyncEngineType::Editor:
		Result = EAvaMediaSyncEngineType::Editor;
		break;
	case EStormSyncEngineType::Game:
		Result = EAvaMediaSyncEngineType::Game;
		break;
	case EStormSyncEngineType::Other:
		Result = EAvaMediaSyncEngineType::Other;
		break;
	case EStormSyncEngineType::Unknown:
		Result = EAvaMediaSyncEngineType::Unknown;
		break;
	default:
		Result = EAvaMediaSyncEngineType::Unknown;
		break;
	}

	return Result;
}

#undef LOCTEXT_NAMESPACE
