// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModularFeature/IAvaMediaSyncProvider.h"
#include "Templates/SharedPointerFwd.h"

enum class EStormSyncEngineType : uint8;
struct FMessageAddress;
struct FStormSyncConnectionInfo;
struct FStormSyncFileDependency;
struct FStormSyncTransportStatusResponse;
struct FStormSyncTransportSyncResponse;

/**
 * Motion Design synchronization provider feature implementation
 */
class FStormSyncAvaSyncProvider : public IAvaMediaSyncProvider
{
public:
	FStormSyncAvaSyncProvider();
	virtual ~FStormSyncAvaSyncProvider() override;

	//~ Begin IAvaMediaSyncProvider interface
	virtual FName GetName() const override;
	virtual void SyncToAll(const TArray<FName>& InPackageNames) override;
	virtual void PushToRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncResponse& DoneDelegate) override;
	virtual void PullFromRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncResponse& DoneDelegate) override;
	virtual void CompareWithRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncCompareResponse& DoneDelegate) override;
	virtual FOnAvaMediaSyncPackageModified& GetOnAvaSyncPackageModified() override;
	//~ End IAvaMediaSyncProvider interface

protected:
	/** Low lvl API using direct message address destination */
	virtual void CompareWith(const FMessageAddress& InRemoteAddress, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncCompareResponse& DoneDelegate);

	/**
	 * Handler called when a file is extracted from an incoming storm sync pak
	 * and relay to the OnAvaSyncPackageModified event
	 */
	void OnPakAssetExtracted(const FName& InPackageName, const FString& InDestFilepath);
	
private:
	FOnAvaMediaSyncPackageModified OnAvaSyncPackageModified;

	/**
	 * Helper to get either client or server user data depending on local playback client / server state
	 *
	 * eg.
	 *
	 * 1. If client, we get user data from server
	 * 2. If server, we get user data from client
	 * 3. In case we are neither a client of server, returns invalid address and false
	 * 4. If an instance is both a client and server, the client takes priority and server side is ignored
	 */
	static bool GetAddressFromUserData(const FString& InRemoteName, const FString& InUserDataKey, FMessageAddress& OutAddress, FText* OutErrorMessage = nullptr);

	/** Helper to get server user data from playback client */
	static bool GetAddressFromServerUserData(const FString& InServerName, const FString& InUserDataKey, FMessageAddress& OutAddress, FText* OutErrorMessage = nullptr);

	/** Helper to get client user data from playback server */
	static bool GetAddressFromClientUserData(const FString& InClientName, const FString& InUserDataKey, FMessageAddress& OutAddress, FText* OutErrorMessage = nullptr);

	/** Returns whether local editor instance has a playback server started */
	static bool IsPlaybackServer();

	/** Returns whether local editor instance has a playback client started */
	static bool IsPlaybackClient();

	/** Helper to create a new error response (used when calling back DoneDelegate to indicate an error) */
	static TSharedPtr<FAvaMediaSyncCompareResponse> CreateErrorResponse(const FText& InText);

	/** Helper to convert a FStormSyncTransportSyncResponse struct to the modular feature equivalent */
	static FAvaMediaSyncResponse ConvertSyncResponse(const TSharedPtr<FStormSyncTransportSyncResponse>& InResponse);

	/** Helper to convert a FStormSyncConnectionInfo struct to the modular feature equivalent */
	static FAvaMediaSyncConnectionInfo ConvertConnectionInfo(const FStormSyncConnectionInfo& InConnectionInfo);

	/** Helper to convert instance type enum specific to this implementation to the interface one (for connection info) */
	static EAvaMediaSyncEngineType ConvertInstanceType(const EStormSyncEngineType InInstanceType);
};
