// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncAvaBridge.h"

#include "Features/IModularFeatures.h"
#include "IAvaMediaModule.h"
#include "IStormSyncTransportClientModule.h"
#include "IStormSyncTransportServerModule.h"
#include "Misc/CoreDelegates.h"
#include "ModularFeature/StormSyncAvaSyncProvider.h"
#include "Playback/IAvaPlaybackServer.h"
#include "Playback/IAvaPlaybackClient.h"
#include "StormSyncAvaBridgeCommon.h"
#include "StormSyncAvaBridgeLog.h"
#include "StormSyncAvaBridgeUtils.h"
#include "StormSyncCoreDelegates.h"

DEFINE_LOG_CATEGORY(LogStormSyncAvaBridge);

#define LOCTEXT_NAMESPACE "FStormSyncAvaBridgeModule"

void FStormSyncAvaBridgeModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStormSyncAvaBridgeModule::OnPostEngineInit);
	FStormSyncCoreDelegates::OnStormSyncServerStarted.AddRaw(this, &FStormSyncAvaBridgeModule::OnStormSyncServerStarted);
	FStormSyncCoreDelegates::OnStormSyncServerStopped.AddRaw(this, &FStormSyncAvaBridgeModule::OnStormSyncServerStopped);

	// Instantiate the sync provider feature on module start
	SyncProvider = MakeUnique<FStormSyncAvaSyncProvider>();

	RegisterConsoleCommands();
}

void FStormSyncAvaBridgeModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FStormSyncCoreDelegates::OnStormSyncServerStarted.RemoveAll(this);
	FStormSyncCoreDelegates::OnStormSyncServerStopped.RemoveAll(this);

	UnregisterConsoleCommands();

	// Unregister the interceptor feature on module shutdown
	if (SyncProvider.IsValid())
	{
		IModularFeatures::Get().UnregisterModularFeature(IAvaMediaSyncProvider::GetModularFeatureName(), SyncProvider.Get());
	}
}

void FStormSyncAvaBridgeModule::OnPostEngineInit()
{
	if (IAvaMediaModule::IsModuleLoaded())
	{
		if (IAvaMediaModule::Get().IsPlaybackServerStarted())
		{
			RegisterUserDataForPlaybackServer();
		}

		if (IAvaMediaModule::Get().IsPlaybackClientStarted())
		{
			RegisterUserDataForPlaybackClient();
		}

		IAvaMediaModule::Get().GetOnAvaPlaybackServerStarted().AddStatic(&FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackServer);
		IAvaMediaModule::Get().GetOnAvaPlaybackClientStarted().AddStatic(&FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackClient);
	}

	if (IStormSyncTransportServerModule::IsAvailable())
	{
		RegisterUserDataForPlaybackServer();
	}

	// Register the sync provider feature
	if (SyncProvider.IsValid())
	{
		IModularFeatures::Get().RegisterModularFeature(IAvaMediaSyncProvider::GetModularFeatureName(), SyncProvider.Get());
	}
}

void FStormSyncAvaBridgeModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("StormSyncAvaBridge.Debug.GetUserData"),
		TEXT("Returns the associated value in Playback Server for ChannelName and Data Key"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&FStormSyncAvaBridgeModule::ExecuteGetUserData),
		ECVF_Default
	));
}

void FStormSyncAvaBridgeModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FStormSyncAvaBridgeModule::ExecuteGetUserData(const TArray<FString>& Args)
{
	const FString Arguments = FString::Join(Args, TEXT(" "));
	UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::ExecuteGetUserData - Args: %ls", *Arguments);

	if (!Args.IsValidIndex(0) || Args[0].IsEmpty())
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "Missing first parameter \"ChannelName\"");
		return;
	}

	if (!Args.IsValidIndex(1) || Args[1].IsEmpty())
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "Missing second parameter \"Key\"");
		return;
	}

	if (!IAvaMediaModule::IsModuleLoaded())
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "Ava Media Module is not available");
		return;
	}

	if (!IAvaMediaModule::Get().IsPlaybackClientStarted())
	{
		UE_LOGF(LogStormSyncAvaBridge, Error, "Ava Playback Client stopped");
		return;
	}

	const FString ChannelName = Args[0];
	const FString Key = Args[1];

	TArray<FString> ServerNames = FStormSyncAvaBridgeUtils::GetServerNamesForChannel(ChannelName);
	if (ServerNames.IsEmpty())
	{
		UE_LOGF(LogStormSyncAvaBridge, Display, "Ava Broadcast profile for channel %ls has no remotes", *ChannelName);
		return;
	}

	const IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	for (const FString& ServerName : ServerNames)
	{
		FString ServerValue = PlaybackClient.GetServerUserData(ServerName, Key);
		UE_LOGF(LogStormSyncAvaBridge, Display, "\t Media Playback Client User Data for server %ls - %ls:%ls", *ServerName, *Key, *ServerValue);
	}
}

void FStormSyncAvaBridgeModule::OnStormSyncServerStarted()
{
	UE_LOGF(LogStormSyncAvaBridge, Verbose, "FStormSyncAvaBridgeModule::OnStormSyncServerStarted");
	RegisterUserDataForPlaybackServer();
}

void FStormSyncAvaBridgeModule::OnStormSyncServerStopped()
{
	UE_LOGF(LogStormSyncAvaBridge, Verbose, "FStormSyncAvaBridgeModule::OnStormSyncServerStopped");
	RegisterUserDataForPlaybackServer();
}

void FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackServer()
{
	// Fail safe checks for both Motion Design and Storm Sync modules, since this method can be executed from delegates in either of these modules
	if (!ValidateModulesAreAvailable())
	{
		return;
	}

	if (IAvaPlaybackServer* PlaybackServer = IAvaMediaModule::Get().GetPlaybackServer())
	{
		const FString StormSyncServerAddress = IStormSyncTransportServerModule::Get().GetServerEndpointMessageAddressId();
		const FString DiscoveryManagerAddress = IStormSyncTransportServerModule::Get().GetDiscoveryManagerMessageAddressId();
		const FString StormSyncClientAddress = IStormSyncTransportClientModule::Get().GetClientEndpointMessageAddressId();

		PlaybackServer->SetUserData(UE::StormSync::AvaBridgeCommon::StormSyncServerAddressKey, StormSyncServerAddress);
		PlaybackServer->SetUserData(UE::StormSync::AvaBridgeCommon::StormSyncClientAddressKey, StormSyncClientAddress);
		PlaybackServer->SetUserData(UE::StormSync::AvaBridgeCommon::StormSyncDiscoveryAddressKey, DiscoveryManagerAddress);

		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackServer StormSyncServerAddress: %ls", *StormSyncServerAddress);
		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackServer StormSyncClientAddress: %ls", *StormSyncClientAddress);
		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackServer DiscoveryManagerAddress: %ls", *DiscoveryManagerAddress);
	}
}

void FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackClient()
{
	// Fail safe checks for both Motion Design and Storm Sync modules, since this method can be executed from delegates in either of these modules
	if (!ValidateModulesAreAvailable())
	{
		return;
	}

	if (IAvaMediaModule::Get().IsPlaybackClientStarted())
	{
		IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();

		const FString StormSyncServerAddress = IStormSyncTransportServerModule::Get().GetServerEndpointMessageAddressId();
		const FString DiscoveryManagerAddress = IStormSyncTransportServerModule::Get().GetDiscoveryManagerMessageAddressId();
		const FString StormSyncClientAddress = IStormSyncTransportClientModule::Get().GetClientEndpointMessageAddressId();

		PlaybackClient.SetUserData(UE::StormSync::AvaBridgeCommon::StormSyncServerAddressKey, StormSyncServerAddress);
		PlaybackClient.SetUserData(UE::StormSync::AvaBridgeCommon::StormSyncClientAddressKey, StormSyncClientAddress);
		PlaybackClient.SetUserData(UE::StormSync::AvaBridgeCommon::StormSyncDiscoveryAddressKey, DiscoveryManagerAddress);

		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackClient StormSyncServerAddress: %ls", *StormSyncServerAddress);
		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackClient StormSyncClientAddress: %ls", *StormSyncClientAddress);
		UE_LOGF(LogStormSyncAvaBridge, Display, "FStormSyncAvaBridgeModule::RegisterUserDataForPlaybackClient DiscoveryManagerAddress: %ls", *DiscoveryManagerAddress);
	}
}

bool FStormSyncAvaBridgeModule::ValidateModulesAreAvailable()
{
	if (!IAvaMediaModule::IsModuleLoaded())
	{
		UE_LOGF(LogStormSyncAvaBridge, Warning, "FStormSyncAvaBridgeModule::ValidateModulesAreAvailable - Failed to set user data cause Ava Media Module is not loaded");
		return false;
	}

	if (!IStormSyncTransportServerModule::IsAvailable())
	{
		UE_LOGF(LogStormSyncAvaBridge, Warning, "FStormSyncAvaBridgeModule::ValidateModulesAreAvailable - Failed to set user data cause Storm Sync Server Module is not loaded");
		return false;
	}

	if (!IStormSyncTransportClientModule::IsAvailable())
	{
		UE_LOGF(LogStormSyncAvaBridge, Warning, "FStormSyncAvaBridgeModule::ValidateModulesAreAvailable - Failed to set user data cause Storm Sync Server Client is not loaded");
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FStormSyncAvaBridgeModule, StormSyncAvaBridge)
