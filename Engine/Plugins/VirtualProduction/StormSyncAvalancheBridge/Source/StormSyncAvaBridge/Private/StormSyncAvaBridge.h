// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class FStormSyncAvaSyncProvider;
class IConsoleObject;

/**
 * Module for Storm Sync Ava Bridge runtime module
 *
 * This class handles the update of User Data in Ava Media Playback Server when module is loaded,
 * and Storm Sync server is started or stopped.
 *
 * This is then replicated and made available on remote clients so that a Playlist asset can associate
 * a message bus address id (for storm sync push / sync) to a remote on a channel.
 */
class FStormSyncAvaBridgeModule : public IModuleInterface
{
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	/** References of registered console commands via IConsoleManager */
	TArray<IConsoleObject*> ConsoleCommands;

	/** Ava sync provider feature instance */
	TUniquePtr<FStormSyncAvaSyncProvider> SyncProvider;

	/** Updates playback user data when engine is fully loaded to setup storm sync message address ids */
	void OnPostEngineInit();

	/** Called from StartupModule and sets up console commands for the plugin via IConsoleManager */
	void RegisterConsoleCommands();

	/** Called from ShutdownModule and clears out previously registered console commands */
	void UnregisterConsoleCommands();

	/** Console command handler to return associated user data in ava playback server */
	static void ExecuteGetUserData(const TArray<FString>& Args);

	/** Updates user data when storm sync server is started */
	void OnStormSyncServerStarted();
	
	/** Updates user data when storm sync server is stopped */
	void OnStormSyncServerStopped();

	/**
	 * Actual helper to set the user data for a running playback server.
	 * 
	 * Sets up Storm Sync Server / Client and Discovery Manager address ids
	 */
	static void RegisterUserDataForPlaybackServer();
	
	/**
	 * Actual helper to set the user data for a running playback client.
	 * 
	 * Sets up Storm Sync Server / Client and Discovery Manager address ids
	 */
	static void RegisterUserDataForPlaybackClient();

	/** Helper ensuring all required modules are loaded available */
	static bool ValidateModulesAreAvailable();
};
