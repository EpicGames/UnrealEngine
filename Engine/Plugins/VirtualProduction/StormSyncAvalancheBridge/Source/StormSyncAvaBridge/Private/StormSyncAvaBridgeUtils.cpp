// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncAvaBridgeUtils.h"
#include "Broadcast/AvaBroadcast.h"

TArray<FString> FStormSyncAvaBridgeUtils::GetServerNamesForChannel(const FString& InChannelName)
{
	TArray<FString> ServerNames;

	if (UAvaBroadcast* Broadcast = UAvaBroadcast::GetBroadcast())
	{
		const FAvaBroadcastOutputChannel Channel = Broadcast->GetCurrentProfile().GetChannel(FName(*InChannelName));
		if (Channel.IsValidChannel())
		{
			TArray<UMediaOutput*> Outputs = Channel.GetMediaOutputs();
			for (const UMediaOutput* Output : Outputs)
			{
				FAvaBroadcastMediaOutputInfo OutputInfo = Channel.GetMediaOutputInfo(Output);
				if (OutputInfo.IsValid() && OutputInfo.IsRemote())
				{
					ServerNames.Add(OutputInfo.ServerName);
				}
			}
		}
	}

	return ServerNames;
}
