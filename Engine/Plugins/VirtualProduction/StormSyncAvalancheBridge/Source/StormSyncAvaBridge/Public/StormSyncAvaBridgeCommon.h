// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::StormSync::AvaBridgeCommon
{
	/** The user data key to store storm sync server address id */
	static constexpr const TCHAR* StormSyncServerAddressKey = TEXT("StormSyncServerAddress");

	/** The user data key to store storm sync discovery manager address id */
	static constexpr const TCHAR* StormSyncClientAddressKey = TEXT("StormSyncClientAddress");

	/** The user data key to store storm sync discovery manager address id */
	static constexpr const TCHAR* StormSyncDiscoveryAddressKey = TEXT("StormSyncDiscoveryAddress");
}
