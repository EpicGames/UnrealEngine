// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"

class STORMSYNCAVABRIDGE_API FStormSyncAvaBridgeUtils
{
public:
	/** Returns the list of Ava Playback Server nmes for a given channel */
	static TArray<FString> GetServerNamesForChannel(const FString& InChannelName);
};
