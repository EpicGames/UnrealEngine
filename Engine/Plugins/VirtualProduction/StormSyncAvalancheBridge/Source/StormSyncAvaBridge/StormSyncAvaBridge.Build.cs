// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StormSyncAvaBridge : ModuleRules
{
	public StormSyncAvaBridge(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheMedia",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"StormSyncCore",
				"StormSyncTransportCore",
				"StormSyncTransportServer",
				"StormSyncTransportClient",
			}
		);
	}
}
