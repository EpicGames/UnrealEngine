// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncAvaBridgeEditor.h"
#include "Modules/ModuleManager.h"
#include "StormSyncAvaBridgeEditorLog.h"
#include "StormSyncAvaRundownExtender.h"

DEFINE_LOG_CATEGORY(LogStormSyncAvaBridgeEditor);

void FStormSyncAvaBridgeEditorModule::StartupModule()
{
	RundownExtender = MakeShared<FStormSyncAvaRundownExtender>();
}

void FStormSyncAvaBridgeEditorModule::ShutdownModule()
{
	RundownExtender.Reset();
}

IMPLEMENT_MODULE(FStormSyncAvaBridgeEditorModule, StormSyncAvaBridgeEditor)
