// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FStormSyncAvaRundownExtender;

class FStormSyncAvaBridgeEditorModule : public IModuleInterface
{
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	/** UI extender for Storm Sync view and commands. */
	TSharedPtr<FStormSyncAvaRundownExtender> RundownExtender;
};
