// Copyright Epic Games, Inc. All Rights Reserved.

#include "StormSyncAvaRundownExtender.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAvaMediaEditorModule.h"
#include "IAvaMediaModule.h"
#include "IStormSyncTransportClientModule.h"
#include "Playback/IAvaPlaybackClient.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditor.h"
#include "StormSyncAvaBridgeCommon.h"
#include "StormSyncAvaBridgeEditorLog.h"
#include "StormSyncAvaBridgeUtils.h"
#include "StormSyncEditor.h"
#include "Subsystems/StormSyncNotificationSubsystem.h"
#include "Toolkits/ToolkitManager.h"

#define LOCTEXT_NAMESPACE "FStormSyncAvaRundownExtender"

FStormSyncAvaRundownExtender::FStormSyncAvaRundownExtender()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FStormSyncAvaRundownExtender::OnPostEngineInit);
}

FStormSyncAvaRundownExtender::~FStormSyncAvaRundownExtender()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (IAvaMediaEditorModule::IsLoaded())
	{
		UnregisterExtension(IAvaMediaEditorModule::Get().GetRundownMenuExtensibilityManager(), MenuExtenderHandle);
		UnregisterExtension(IAvaMediaEditorModule::Get().GetRundownToolBarExtensibilityManager(), ToolbarExtenderHandle);
	}
}

void FStormSyncAvaRundownExtender::OnPostEngineInit()
{
	if (IAvaMediaEditorModule::IsLoaded())
	{
		RegisterMenuExtensions();
	}
}

void FStormSyncAvaRundownExtender::RegisterMenuExtensions()
{
	// Context menu extension
	MenuExtenderHandle = RegisterExtension(
		IAvaMediaEditorModule::Get().GetRundownMenuExtensibilityManager(),
		FAssetEditorExtender::CreateSP(this, &FStormSyncAvaRundownExtender::AddMenuExtender)
	);

	// Toolbar extension
	ToolbarExtenderHandle = RegisterExtension(
		IAvaMediaEditorModule::Get().GetRundownToolBarExtensibilityManager(),
		FAssetEditorExtender::CreateSP(this, &FStormSyncAvaRundownExtender::AddToolbarExtender)
	);
}

FDelegateHandle FStormSyncAvaRundownExtender::RegisterExtension(const TSharedPtr<FExtensibilityManager> InExtensibilityManager, const FAssetEditorExtender& InExtenderDelegate)
{
	if (!InExtensibilityManager.IsValid())
	{
		const FDelegateHandle DelegateHandle;
		return DelegateHandle;
	}

	const int32 ExtenderIndex = InExtensibilityManager->GetExtenderDelegates().Add(InExtenderDelegate);
	return InExtensibilityManager->GetExtenderDelegates()[ExtenderIndex].GetHandle();
}

void FStormSyncAvaRundownExtender::UnregisterExtension(const TSharedPtr<FExtensibilityManager> InExtensibilityManager, const FDelegateHandle& InHandleToRemove)
{
	if (!InExtensibilityManager.IsValid())
	{
		return;
	}

	InExtensibilityManager->GetExtenderDelegates().RemoveAll([Handle = InHandleToRemove](const FAssetEditorExtender& Extender)
	{
		return Handle == Extender.GetHandle();
	});
}

TSharedRef<FExtender> FStormSyncAvaRundownExtender::AddMenuExtender(const TSharedRef<FUICommandList> InCommandList, const TArray<UObject*> ContextSensitiveObjects)
{
	UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "FStormSyncAvaRundownExtender::AddMenuExtender - Adding in menu extensions");

	TSharedRef<FExtender> Extender(new FExtender());

	const UAvaRundown* Rundown = ContextSensitiveObjects.IsValidIndex(0) ? Cast<UAvaRundown>(ContextSensitiveObjects[0]) : nullptr;
	if (!Rundown)
	{
		return Extender;
	}

	const TSharedPtr<IToolkit> AssetEditor = FToolkitManager::Get().FindEditorForAsset(Rundown);
	if (!AssetEditor.IsValid())
	{
		return Extender;
	}

	const TSharedPtr<FAvaRundownEditor> RundownEditor = StaticCastSharedPtr<FAvaRundownEditor>(AssetEditor);
	if (!RundownEditor.IsValid())
	{
		return Extender;
	}

	// Template panel extension
	Extender->AddMenuExtension(
		MenuExtensionHook,
		EExtensionHook::After,
		InCommandList,
		// Convert to weak ptr to prevent ownership to the rundown editor and potentially increasing its lifetime
		FMenuExtensionDelegate::CreateSP(this, &FStormSyncAvaRundownExtender::CreateTemplateContextMenu, Rundown, TWeakPtr<FAvaRundownEditor>(RundownEditor))
	);

	return Extender;
}

void FStormSyncAvaRundownExtender::CreateTemplateContextMenu(FMenuBuilder& MenuBuilder, const UAvaRundown* InRundown, TWeakPtr<FAvaRundownEditor> InRundownEditor)
{
	check(InRundown);

	const TSharedPtr<FAvaRundownEditor> RundownEditor = InRundownEditor.Pin();
	if (!RundownEditor.IsValid())
	{
		UE_LOGF(LogStormSyncAvaBridgeEditor, Error, "FStormSyncAvaRundownExtender::CreateTemplateContextMenu - Invalid shared ptr from weak ptr delegate param");
		return;
	}

	if (!FModuleManager::Get().IsModuleLoaded(TEXT("StormSyncEditor")))
	{
		UE_LOGF(LogStormSyncAvaBridgeEditor, Error, "FStormSyncAvaRundownExtender::CreateTemplateContextMenu - StormSyncEditor module is not loaded. Is StormSync plugin enabled ?");
		return;
	}

	FText DisabledTooltipReason;
	bool bIsValidSelection = false;
	TArray<FName> SelectedPackageNames;
	FCanExecuteAction DefaultCanExecuteAction;
	GetContextMenuSelectionInfos(InRundown, InRundownEditor, bIsValidSelection, DisabledTooltipReason, SelectedPackageNames, DefaultCanExecuteAction);

	// For later use with action handler
	const FStormSyncEditorModule& StormSyncEditor = FStormSyncEditorModule::Get();

	MenuBuilder.BeginSection("StormSyncOperations_Template", LOCTEXT("StormSyncOperationsHeader", "Synchronize Actions"));

	// Initialize action
	{
		const FText TooltipText = bIsValidSelection ?
			FText::Format(LOCTEXT("Initialize_Tooltip", "Sync asset over remote node.{0}"), DisabledTooltipReason) :
			FText::Format(LOCTEXT("Initialize_Tooltip_Invalid", "Please ensure the rundown page is using a valid Motion Design Asset.{0}"), DisabledTooltipReason);

		const TArray<FAvaRundownPage> SelectedTemplatePages = GetSelectedPages(InRundown, InRundownEditor);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Initialize_Label", "Initialize"),
			TooltipText,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FStormSyncAvaRundownExtender::HandleInitializeAction, InRundown, SelectedTemplatePages),
				DefaultCanExecuteAction
			)
		);
	}

	constexpr bool bOpenSubMenuOnClick = false;

	// Push action
	{
		FText LabelText;
		FText TooltipText;

		if (!bIsValidSelection)
		{
			LabelText = LOCTEXT("PushAssetsMenuEntryInvalid", "Cannot push. Page has no valid asset.");
			TooltipText = LOCTEXT("PushAssetsMenuEntryTooltipInvalid", "Please ensure the rundown pages are using a valid Motion Design Asset.");
		}
		else
		{
			LabelText = FText::Format(
				LOCTEXT("PushAssetsAssetName", "Push {0} selected asset{1} to"),
				FText::AsNumber(SelectedPackageNames.Num()),
				FText::FromString(SelectedPackageNames.Num() == 1 ? TEXT("") : TEXT("s"))
			);

			TooltipText = FText::Format(
				LOCTEXT("PushAssetsMenuEntryTooltip", "Push {0} selected asset{1} (and inner dependencies) to specific remote.\n\nTransfer will only proceed if changes are detected.{2}"),
				FText::AsNumber(SelectedPackageNames.Num()),
				FText::FromString(SelectedPackageNames.Num() == 1 ? TEXT("") : TEXT("s")),
				DisabledTooltipReason
			);
		}

		constexpr bool bIsPushing = true;
		MenuBuilder.AddSubMenu(
			LabelText,
			TooltipText,
			FNewMenuDelegate::CreateRaw(
				&StormSyncEditor,
				&FStormSyncEditorModule::BuildPushAssetsMenuSection,
				SelectedPackageNames,
				bIsPushing
			),
			FUIAction(FExecuteAction(), DefaultCanExecuteAction),
			NAME_None,
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll")
		);
	}

	// Compare action
	{
		FText LabelText;
		FText TooltipText;

		if (!bIsValidSelection)
		{
			LabelText = LOCTEXT("CompareAssetsMenuEntryInvalid", "Cannot compare. Page has no valid asset.");
			TooltipText = LOCTEXT("CompareAssetsMenuEntryTooltipInvalid", "Please ensure the rundown pages are using a valid Motion Design Asset.");
		}
		else
		{
			LabelText = LOCTEXT("CompareAssetsMenuEntry", "Compare Asset(s) with");
			TooltipText = FText::Format(
				LOCTEXT("CompareAssetsRemoteMenuEntry", "Compare asset(s) with a specific remote and see if files (and inner dependencies) are either missing or in mismatched state.{0}"),
				DisabledTooltipReason
			);
		}

		MenuBuilder.AddSubMenu(
			LabelText,
			TooltipText,
			FNewMenuDelegate::CreateRaw(
				&StormSyncEditor,
				&FStormSyncEditorModule::BuildCompareWithMenuSection,
				SelectedPackageNames
			),
			FUIAction(FExecuteAction(), DefaultCanExecuteAction),
			NAME_None,
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Visualizers")
		);
	}
	MenuBuilder.EndSection();
}

void FStormSyncAvaRundownExtender::HandleInitializeAction(const UAvaRundown* InRundown, TArray<FAvaRundownPage> InSelectedTemplatePages)
{
	// Build a list of package names to push grouped by channel name (and remote)
	// Key is the remote address id, value is the list of package names to synchronize
	TMap<FString, TArray<FName>> PackageNamesPerChannel;

	for (const FAvaRundownPage& SelectedPage : InSelectedTemplatePages)
	{
		const FString AssetName = SelectedPage.GetAssetPath(InRundown).GetLongPackageName();
		TArray<FString> ChannelNames = GetChannelNamesForTemplatePage(InRundown, SelectedPage);

		// From the list of channel names that match this asset to sync, build the list of server names
		// Note: there may be more than one server per channel (channel support multiple outputs).

		UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "ChannelNames for asset \"%ls\" are: %ls", *AssetName, *FString::Join(ChannelNames, TEXT(", ")));

		TArray<FString> RemoteServerNames;
		for (FString ChannelName : ChannelNames)
		{
			const TArray<FString> ServerNames = FStormSyncAvaBridgeUtils::GetServerNamesForChannel(ChannelName);
			if (ServerNames.IsEmpty())
			{
				UE_LOGF(LogStormSyncAvaBridgeEditor, Warning, "FStormSyncAvaRundownExtender::HandleInitializeActions - Unable to determine playback servers for channel \"%ls\"", *ChannelName);
				continue;
			}

			RemoteServerNames.Append(ServerNames);
		}

		if (IAvaMediaModule::Get().IsPlaybackClientStarted())
		{
			const IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
			for (const FString& ServerName : RemoteServerNames)
			{
				// Get address id for storm sync client on playback host
				FString ClientAddress = PlaybackClient.GetServerUserData(ServerName, UE::StormSync::AvaBridgeCommon::StormSyncClientAddressKey);
				UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "ClientAddress on playback server %ls is %ls", *ServerName, *ClientAddress);

				if (ClientAddress.IsEmpty())
				{
					UE_LOGF(LogStormSyncAvaBridgeEditor, Warning, "Storm Sync Server Adress id for playback server %ls is empty", *ServerName);
					continue;
				}

				TArray<FName>& PackageNames = PackageNamesPerChannel.FindOrAdd(ClientAddress);
				PackageNames.Add(FName(*AssetName));
			}
		}
	}

	for (const TPair<FString, TArray<FName>>& NamesPerChannel : PackageNamesPerChannel)
	{
		FString AddressId = NamesPerChannel.Key;
		TArray<FName> PackageNames = NamesPerChannel.Value;

		UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "Names per channel - Channel: %ls", *AddressId);

		for (const FName& PackageName : PackageNames)
		{
			UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "\t PackageName: %ls", *PackageName.ToString());
		}

		PushPackagesToRemote(AddressId, PackageNames);
	}
}

TSharedRef<FExtender> FStormSyncAvaRundownExtender::AddToolbarExtender(const TSharedRef<FUICommandList> InCommandList, const TArray<UObject*> ContextSensitiveObjects)
{
	UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "FStormSyncAvaRundownExtender::AddToolbarExtender - Adding in toolbar extensions");

	TSharedRef<FExtender> Extender(new FExtender());

	const UAvaRundown* Rundown = ContextSensitiveObjects.IsValidIndex(0) ? Cast<UAvaRundown>(ContextSensitiveObjects[0]) : nullptr;
	if (!Rundown)
	{
		return Extender;
	}

	const TSharedPtr<IToolkit> AssetEditor = FToolkitManager::Get().FindEditorForAsset(Rundown);
	if (!AssetEditor.IsValid())
	{
		return Extender;
	}

	const TSharedPtr<FAvaRundownEditor> RundownEditor = StaticCastSharedPtr<FAvaRundownEditor>(AssetEditor);
	if (!RundownEditor.IsValid())
	{
		return Extender;
	}

	Extender->AddToolBarExtension(
		"Pages",
		EExtensionHook::After,
		InCommandList,
		FToolBarExtensionDelegate::CreateSP(this, &FStormSyncAvaRundownExtender::FillToolbar, TWeakPtr<FAvaRundownEditor>(RundownEditor))
	);

	UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "FStormSyncAvaRundownExtender::AddToolbarExtender - Rundown: %ls", *GetNameSafe(Rundown));

	return Extender;
}

void FStormSyncAvaRundownExtender::FillToolbar(FToolBarBuilder& ToolbarBuilder, TWeakPtr<FAvaRundownEditor> InRundownEditor)
{
	ToolbarBuilder.BeginSection(TEXT("StormSync"));
	{
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &FStormSyncAvaRundownExtender::GenerateToolbarMenu, InRundownEditor),
			LOCTEXT("ToolbarLabel", "Synchronize Actions"),
			LOCTEXT("ToolbarToolTip", "Synchronize the rundown assets over the network"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Recompile"),
			false
		);
	}
	ToolbarBuilder.EndSection();
}

TSharedRef<SWidget> FStormSyncAvaRundownExtender::GenerateToolbarMenu(TWeakPtr<FAvaRundownEditor> InRundownEditor)
{
	TArray<FName> PackageNames;

	const TSharedPtr<FAvaRundownEditor> RundownEditor = InRundownEditor.Pin();

	if (RundownEditor.IsValid() && RundownEditor->IsRundownValid())
	{
		const UAvaRundown* Rundown = RundownEditor->GetRundown(); 

		// Gather the list of all package names from Motion Design assets in this Rundown pages
		const FAvaRundownPageCollection& PageCollection = Rundown->GetTemplatePages();
		for (const FAvaRundownPage& Page : PageCollection.Pages)
		{
			if (FString PackageName = Page.GetAssetPath(Rundown).GetLongPackageName(); !PackageName.IsEmpty())
			{
				PackageNames.AddUnique(FName(*PackageName));
			}
		}
	}

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(TEXT("StormSyncActions"));

	if (FModuleManager::Get().IsModuleLoaded(TEXT("StormSyncEditor")))
	{
		FStormSyncEditorModule& StormSyncEditor = FStormSyncEditorModule::Get();

		const int32 PackagesCount = PackageNames.Num();

		TArray<FString> AssetList;
		for (const FName PackageName : PackageNames)
		{
			AssetList.Add(FString::Printf(TEXT("- %s"), *PackageName.ToString()));
		}

		FText LabelText = FText::Format(LOCTEXT("PushAssetsToolbarMenuEntry", "Push {0} asset(s) to"), FText::AsNumber(PackagesCount));
		FText TooltipText = FText::Format(LOCTEXT(
			"PushAssetsToolbarMenuEntryTooltip",
			"Push {0} asset(s) (and inner dependencies) to specific remote.\n\nTransfer will only proceed if changes are detected.\n\n{1}"
		), FText::AsNumber(PackagesCount), FText::FromString(FString::Join(AssetList, LINE_TERMINATOR)));

		const bool bIsPushEnabled = !PackageNames.IsEmpty();
		if (!bIsPushEnabled)
		{
			LabelText = LOCTEXT("PushAssetsToolbarMenuEntryInvalid", "Cannot push. Rundown pages have no valid assets.");
			TooltipText = LOCTEXT("PushAssetsToolbarMenuEntryTooltipInvalid", "Please ensure the rundown pages are using a valid Motion Design Asset");
		}

		constexpr bool bIsPushing = true;
		constexpr bool bOpenSubMenuOnClick = false;
		MenuBuilder.AddSubMenu(
			LabelText,
			TooltipText,
			FNewMenuDelegate::CreateRaw(
				&StormSyncEditor,
				&FStormSyncEditorModule::BuildPushAssetsMenuSection,
				PackageNames,
				bIsPushing
			),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([bIsPushEnabled]() { return bIsPushEnabled; })
			),
			NAME_None,
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll")
		);

		MenuBuilder.AddSubMenu(
			FText::Format(LOCTEXT("CompareAssetsToolbarMenuEntry", "Compare {0} asset(s) with"), FText::AsNumber(PackagesCount)),
			LOCTEXT("CompareAssetsToolbarMenuEntryTooltip", "Compare asset(s) with a specific remote and see if files (and inner dependencies) are either missing or in mismatched state."),
			FNewMenuDelegate::CreateRaw(
				&StormSyncEditor,
				&FStormSyncEditorModule::BuildCompareWithMenuSection,
				PackageNames
			),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda([bIsPushEnabled]() { return bIsPushEnabled; })
			),
			NAME_None,
			EUserInterfaceActionType::Button,
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Visualizers")
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FStormSyncAvaRundownExtender::PushPackagesToRemote(const FString& RemoteAddressId, const TArray<FName>& InPackageNames)
{
	UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "FStormSyncAvaRundownExtender::PushPackagesToRemote - InSelectedPage: (InPackageNames: %d, RemoteAddressId: %ls)", InPackageNames.Num(), *RemoteAddressId);

	FMessageAddress RemoteMessageAddress;
	if (!FMessageAddress::Parse(RemoteAddressId, RemoteMessageAddress))
	{
		UE_LOGF(LogStormSyncAvaBridgeEditor, Error, "Unable to parse %ls into a Message Address", *RemoteAddressId);
		return;
	}

	// Note: We sync with a dummy package descriptor, next iterations could add in there an additional UI step.
	// Something that could be handled with a bit more UI integration, like some kind of popup window or wizard.
	const FStormSyncPackageDescriptor PackageDescriptor;

	const FOnStormSyncPushComplete Delegate = FOnStormSyncPushComplete::CreateLambda([](const TSharedPtr<FStormSyncTransportPushResponse>& Response)
	{
		check(Response.IsValid())
		UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "FStormSyncAvaRundownExtender::PushPackagesToRemote - Got a response: %ls", *Response->ToString());
		UStormSyncNotificationSubsystem::Get().HandlePushResponse(Response);
	});

	IStormSyncTransportClientModule::Get().PushPackages(PackageDescriptor, InPackageNames, RemoteMessageAddress, Delegate);
}

TArray<FName> FStormSyncAvaRundownExtender::GetSelectedPackagesNames(const UAvaRundown* InRundown, const TWeakPtr<FAvaRundownEditor>& InRundownEditor)
{
	check(InRundown);

	TArray<FName> Result;

	const TArray<FAvaRundownPage> SelectedPages = GetSelectedPages(InRundown, InRundownEditor);
	Algo::Transform(SelectedPages, Result, [InRundown](const FAvaRundownPage& Page)
	{
		return Page.GetAssetPath(InRundown).GetLongPackageFName();
	});

	return Result;
}

TArray<FAvaRundownPage> FStormSyncAvaRundownExtender::GetSelectedPages(const UAvaRundown* InRundown, const TWeakPtr<FAvaRundownEditor>& InRundownEditor)
{
	check(InRundown);

	TArray<FAvaRundownPage> Result;
	const TSharedPtr<FAvaRundownEditor> RundownEditor = InRundownEditor.Pin();
	if (!RundownEditor.IsValid())
	{
		UE_LOGF(LogStormSyncAvaBridgeEditor, Display, "FStormSyncAvaRundownExtender::GetSelectedPages - Invalid shared ptr from weak ptr delegate param");
		return Result;
	}

	const TConstArrayView<int32> SelectedPageIds = RundownEditor->GetSelectedPagesOnFocusedWidget();
	for (const int32 SelectedPageId : SelectedPageIds)
	{
		FAvaRundownPage Page = InRundown->GetPage(SelectedPageId);
		if (!Page.IsValidPage())
		{
			continue;
		}

		FSoftObjectPath SoftAssetPath = Page.GetAssetPath(InRundown);
		FString LongPackageName = SoftAssetPath.GetLongPackageName();
		FString AssetName = SoftAssetPath.GetAssetName();

		if (!LongPackageName.IsEmpty())
		{
			Result.Add(Page);
		}
	}

	return Result;
}

TArray<FString> FStormSyncAvaRundownExtender::GetChannelNamesForTemplatePage(const UAvaRundown* InRundown, const FAvaRundownPage& InTemplatePage)
{
	check(InRundown);

	TArray<FString> ChannelNames;

	// Here, we try to determine the list of channels to consider for a sync operation, from the selected template page,
	// with instanced pages that are using the selected package name (Motion Design asset)
	const FString PackageName = InTemplatePage.GetAssetPath(InRundown).GetLongPackageName();

	// Build the list of instanced pages matching the asset name we want to sync
	TArray<FAvaRundownPage> Pages = InRundown->GetInstancedPages().Pages.FilterByPredicate([InRundown, PackageName](const FAvaRundownPage& Page)
	{
		return Page.GetAssetPath(InRundown).GetLongPackageName() == PackageName;
	});

	// From there, build a unique list of channel outputs
	for (const FAvaRundownPage& Page : Pages)
	{
		ChannelNames.AddUnique(Page.GetChannelName().ToString());
	}

	return ChannelNames;
}

bool FStormSyncAvaRundownExtender::GetContextMenuSelectionInfos(const UAvaRundown* InRundown, const TWeakPtr<FAvaRundownEditor>& InRundownEditor, bool& bOutIsValidSelection, FText& OutDisabledReasonTooltip, TArray<FName>& OutSelectedPackageNames, FCanExecuteAction& OutCanExecuteAction)
{
	if (!FModuleManager::Get().IsModuleLoaded(TEXT("StormSyncEditor")))
	{
		UE_LOGF(LogStormSyncAvaBridgeEditor, Error, "FStormSyncAvaRundownExtender::GetContextMenuSelectionInfos - StormSyncEditor module is not loaded. Is StormSync plugin enabled ?");
		return false;
	}

	const FStormSyncEditorModule& StormSyncEditor = FStormSyncEditorModule::Get();

	TArray<FName> SelectedPackagesNames = GetSelectedPackagesNames(InRundown, InRundownEditor);
	bool bIsValidSelection = !SelectedPackagesNames.IsEmpty();

	// Figure out if selection is containing dirty (unsaved) assets
	FText DisabledTooltipReason;
	const TArray<FAssetData> DirtyAssets = StormSyncEditor.GetDirtyAssets(SelectedPackagesNames, DisabledTooltipReason);

	bool bContainsDirtyAssets = !DirtyAssets.IsEmpty();
	FCanExecuteAction DefaultCanExecuteAction = FCanExecuteAction::CreateLambda([bIsValidSelection, bContainsDirtyAssets]() { return bIsValidSelection && !bContainsDirtyAssets; });

	bOutIsValidSelection = bIsValidSelection;
	OutDisabledReasonTooltip = DisabledTooltipReason;
	OutSelectedPackageNames = MoveTemp(SelectedPackagesNames);
	OutCanExecuteAction = MoveTemp(DefaultCanExecuteAction);

	return true;
}

#undef LOCTEXT_NAMESPACE
