// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MessageEndpoint.h"
#include "Toolkits/AssetEditorToolkit.h"

class FAvaRundownEditor;
class FExtensibilityManager;
class UAvaRundown;
struct FAvaRundownPage;

/** This class handles Motion Design Rundown assets UI extensions for context menu / toolbars */
class FStormSyncAvaRundownExtender : public TSharedFromThis<FStormSyncAvaRundownExtender>
{
public:
	FStormSyncAvaRundownExtender();
	~FStormSyncAvaRundownExtender();

private:
	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Name of the extension point we're providing extension next to */
	static constexpr const TCHAR* MenuExtensionHook = TEXT("PageListOperations");
	
	/** Menu extender for motion design rundown editor context menu */
	FDelegateHandle MenuExtenderHandle;
	
	/** Toolbar extender for motion design rundown editor */
	FDelegateHandle ToolbarExtenderHandle;

	/** Register Context menu / toolbar extensions when editor is fully loaded */
	void OnPostEngineInit();
	
	/** Startup module handler for UI extensions */
	void RegisterMenuExtensions();

	/** Startup module handler to register an UI extensions from one of Motion Design exposed extensibility hooks */
	static FDelegateHandle RegisterExtension(const TSharedPtr<FExtensibilityManager> InExtensibilityManager, const FAssetEditorExtender& InExtenderDelegate);
	
	/** Shutdown module handler to unregister any UI extensions added here */
	static void UnregisterExtension(const TSharedPtr<FExtensibilityManager> InExtensibilityManager, const FDelegateHandle& InHandleToRemove);
	
	/** Gets the extender to use for rundown context sensitive menus */
	TSharedRef<FExtender> AddMenuExtender(const TSharedRef<FUICommandList> InCommandList, const TArray<UObject*> ContextSensitiveObjects);

	/* UI Menu Extension handler for template panel */
	void CreateTemplateContextMenu(FMenuBuilder& MenuBuilder, const UAvaRundown* InRundown, TWeakPtr<FAvaRundownEditor> InRundownEditor);

	/** Context menu handler for initialize action */
	void HandleInitializeAction(const UAvaRundown* InRundown, TArray<FAvaRundownPage> InSelectedTemplatePages);
	
	/** Gets the extender to use for rundown context sensitive menus */
	TSharedRef<FExtender> AddToolbarExtender(const TSharedRef<FUICommandList> InCommandList, const TArray<UObject*> ContextSensitiveObjects);
	
	/** Construct toolbar widgets for rundown sync actions */
	void FillToolbar(FToolBarBuilder& ToolbarBuilder, TWeakPtr<FAvaRundownEditor> InRundownEditor);

	/** Creates widget for toolbar content */
	TSharedRef<SWidget> GenerateToolbarMenu(TWeakPtr<FAvaRundownEditor> InRundownEditor);

	/** Storm sync push a list of package names to a remote address ID */
	static void PushPackagesToRemote(const FString& RemoteAddressId, const TArray<FName>& InPackageNames);

	/** Gather a list of package names from currently selected pages in editor */
	static TArray<FName> GetSelectedPackagesNames(const UAvaRundown* InRundown, const TWeakPtr<FAvaRundownEditor>& InRundownEditor);
	
	/**
	 * Returns the list of currently selected pages in editor
	 *
	 * It will only return selection if the page is referencing a valid Motion Design Asset (not "None" path)
	 */
	static TArray<FAvaRundownPage> GetSelectedPages(const UAvaRundown* InRundown, const TWeakPtr<FAvaRundownEditor>& InRundownEditor);

	/** Returns a unique list of channel names, gather from instanced pages, that are matching the passed in template page selection and selected asset name */
	static TArray<FString> GetChannelNamesForTemplatePage(const UAvaRundown* InRundown, const FAvaRundownPage& InTemplatePage);

	/**
	 * Provides a way for context menu builders to gather information about current selection.
	 *
	 * @param InRundown The rundown UObject we're working with
	 * @param InRundownEditor The rundown asset editor we're working with
	 * @param bOutIsValidSelection Output indicating whether selection is valid - eg. current selection returns a list of valid package names
	 * @param OutDisabledReasonTooltip Output holding the tooltip to use in case the action is disabled - eg. in case the selection contains unsaved dirty assets
	 * @param OutSelectedPackageNames Output list of package names of the selection, as returned by GetSelectedPackagesNames()
	 * @param OutCanExecuteAction Output delegate to use as a default CanExecuteAction - eg. disabling the action if selection is invalid
	 *
	 * @return false if StormSyncEditor module is not available, and we failed to determine selection state
	 */
	static bool GetContextMenuSelectionInfos(
		const UAvaRundown* InRundown,
		const TWeakPtr<FAvaRundownEditor>& InRundownEditor,
		bool& bOutIsValidSelection,
		FText& OutDisabledReasonTooltip,
		TArray<FName>& OutSelectedPackageNames,
		FCanExecuteAction& OutCanExecuteAction
	);
};
