// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class StormSyncAvaBridgeEditor : ModuleRules
{
	public StormSyncAvaBridgeEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheMedia",
				"AvalancheMediaEditor",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"Slate",
				"SlateCore",
				"StormSyncAvaBridge",
				"StormSyncCore",
				"StormSyncEditor",
				"StormSyncTransportCore",
				"StormSyncTransportClient",
				"StormSyncTransportServer",
			}
		);

		ShortName = "StScAvBridEd";
	}
}
