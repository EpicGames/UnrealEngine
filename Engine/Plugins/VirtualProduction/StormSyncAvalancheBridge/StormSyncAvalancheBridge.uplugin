{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Storm Sync Motion Design Bridge",
	"Description": "Plugin bridge between Motion Design Plugin and Storm Sync to provide in-editor integration to synchronize assets",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "StormSyncAvaBridgeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "StormSyncAvaBridge",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "Avalanche",
			"Enabled": true
		},
		{
			"Name": "StormSync",
			"Enabled": true
		}
	]
}
